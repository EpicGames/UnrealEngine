// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;
using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class NNEEditor : ModuleRules
{
	public NNEEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange
			(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"DeveloperSettings",
				"Slate",
				"SlateCore",
				"UnrealEd",
				"AssetTools",
				"NNE"
			}
		);

		string SharedLibPath = Path.Combine(ModuleDirectory, "Bin", Target.Platform.ToString());

		if (Target.Platform == UnrealTargetPlatform.Win64)
		{
			string ArchPath = (Target.Architecture == UnrealArch.Arm64) ? "arm64/" : "";
			string SharedLibFileName = "NNEEditorOnnxTools.dll";

			PublicDelayLoadDLLs.Add(SharedLibFileName);

			RuntimeDependencies.Add("$(TargetOutputDir)/"+ ArchPath + SharedLibFileName, Path.Combine(SharedLibPath, ArchPath + SharedLibFileName));

			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SUPPORTED");
			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SHAREDLIB_FILENAME=" + SharedLibFileName);
		}
		else if (Target.Platform == UnrealTargetPlatform.Linux)
		{
			string SharedLibFileName = "libNNEEditorOnnxTools.so";

			PublicDelayLoadDLLs.Add(SharedLibFileName);

			RuntimeDependencies.Add("$(TargetOutputDir)/" + SharedLibFileName, Path.Combine(SharedLibPath, SharedLibFileName));

			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SUPPORTED");
			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SHAREDLIB_FILENAME=" + SharedLibFileName);
		}
		else if (Target.Platform == UnrealTargetPlatform.Mac)
		{
			string SharedLibFileName = "libNNEEditorOnnxTools.dylib";

			// Not fully supported on macOS:
			// https://dev.epicgames.com/documentation/en-us/unreal-engine/integrating-third-party-libraries-into-unreal-engine
			// PublicDelayLoadDLLs.Add(SharedLibFileName);

			RuntimeDependencies.Add("$(TargetOutputDir)/" + SharedLibFileName, Path.Combine(SharedLibPath, SharedLibFileName));

			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SUPPORTED");
			PublicDefinitions.Add("NNEEDITORONNXTOOLS_SHAREDLIB_FILENAME=" + SharedLibFileName);
		}

		PublicDefinitions.Add("UE_NNEEDITORONNXTOOLS");
	}
}
