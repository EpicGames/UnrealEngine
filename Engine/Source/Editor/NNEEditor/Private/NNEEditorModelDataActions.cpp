// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEEditorModelDataActions.h"

#include "NNE.h"
#include "NNEEditorModelDataEditorToolkit.h"
#include "NNEModelData.h"

#define LOCTEXT_NAMESPACE "AssetTypeActions"

namespace UE::NNEEditor::Private
{

	UClass* FModelDataAssetTypeActions::GetSupportedClass() const
	{
		return UNNEModelData::StaticClass();
	}

	FText FModelDataAssetTypeActions::GetName() const
	{
		return LOCTEXT("FModelDataAssetTypeActionsName", "NNE Model Data");
	}

	FColor FModelDataAssetTypeActions::GetTypeColor() const
	{
		return FColor::Cyan;
	}

	uint32 FModelDataAssetTypeActions::GetCategories()
	{
		return EAssetTypeCategories::Misc;
	}

	void FModelDataAssetTypeActions::OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor)
	{
		TArray<UNNEModelData*> Models;
		for (UObject* Object : InObjects)
		{
			UNNEModelData* Model = Cast<UNNEModelData>(Object);
			if (Model)
			{
				Models.Add(Model);
			}
			else
			{
				UE_LOGF(LogNNE, Warning, "Casting asset to UNNEModelData failed");
			}
		}
		if (!Models.IsEmpty())
		{
			MakeShared<FModelDataEditorToolkit>()->InitEditor(Models);
		}
	}

} // UE::NNEEditor::Private

#undef LOCTEXT_NAMESPACE