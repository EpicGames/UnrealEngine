// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "AssetTypeActions_Base.h"

class UNNEModelData;

namespace UE::NNEEditor::Private
{
	class FModelDataAssetTypeActions : public FAssetTypeActions_Base
	{
	public:
		// IAssetTypeActions interface
		virtual FText GetName() const override;
		virtual FColor GetTypeColor() const override;
		virtual UClass* GetSupportedClass() const override;
		virtual uint32 GetCategories() override;
		virtual void OpenAssetEditor(const TArray<UObject*>& InObjects, TSharedPtr<class IToolkitHost> EditWithinLevelEditor) override;
		// End of IAssetTypeActions interface
	};

} // UE::NNEEditor::Private