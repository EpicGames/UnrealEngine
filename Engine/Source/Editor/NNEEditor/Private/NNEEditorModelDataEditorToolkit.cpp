// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEEditorModelDataEditorToolkit.h"

#include "Kismet/KismetSystemLibrary.h"
#include "Modules/ModuleManager.h"
#include "NNE.h"
#include "NNEModelData.h"
#include "NNERuntime.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SBoxPanel.h"


namespace UE::NNEEditor::Private
{
	class SRuntimeExpandableArea : public SExpandableArea
	{

	public:

		TAttribute<bool> IsExpandable;

		void Construct(const SExpandableArea::FArguments& InArgs)
		{
			SExpandableArea::Construct(InArgs);
			const FSlateColor TintColor(FLinearColor(FVector4f(0, 0, 0, 0)));
			EmptyImage = *ExpandedImage;
			EmptyImage.TintColor = TintColor;
		}

	protected:

		const FSlateBrush* OnGetCollapseImage() const override
		{
			if (IsExpandable.Get())
			{
				return bAreaCollapsed ? CollapsedImage : ExpandedImage;
			}
			return &EmptyImage;
		}

		void OnToggleContentVisibility() override
		{
			if (IsExpandable.Get())
			{
				SExpandableArea::OnToggleContentVisibility();
			}
		}

	private:

		FSlateBrush EmptyImage;

	};

	static void AppendUniqueRuntimes(TArray<FString>& InOutDest, const TArrayView<const FString>& InSource)
	{
		for (const FString& RuntimeName : InSource)
		{
			InOutDest.AddUnique(RuntimeName);
		}
	}

	void FModelDataEditorToolkit::InitEditor(const TArray<UNNEModelData*>& InModels)
	{
		checkf(InModels.Num() > 0, TEXT("FModelDataEditorToolkit require at least one item in InModels"))
		Models = InModels;
		AllRuntimeNames = UE::NNE::GetAllRuntimeNames();
		//A runtime could possibly be in the target list in any of the models but not be registered to the editor
		for (UNNEModelData* Model : Models)
		{
			AppendUniqueRuntimes(AllRuntimeNames, Model->GetTargetRuntimes());
		}
		EmptySelectionModels = NewObject<UNNEEditorEmptySelectionModels>();
		EmptySelectionModels->AddToRoot();
		EmptySelectionModels->SetFlags(RF_Transient | RF_Transactional);
		FDetailsViewArgs DetailsViewArgs{};
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bShowObjectLabel = true;
		DetailsViewArgs.bShowSectionSelector = false;
		DetailsViewArgs.bCustomNameAreaLocation = true;
		Runtimes.Reset(AllRuntimeNames.Num());
		for(const FString& RuntimeName : AllRuntimeNames)
		{
			TSharedRef<IDetailsView> View = FModuleManager::LoadModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
			View->OnFinishedChangingProperties().AddRaw(this, &FModelDataEditorToolkit::OnRuntimePropertyChanged);
			Runtimes.Emplace(RuntimeName, View);
		}
		ReloadSettings();
		PostUndoRedoHandle = FEditorDelegates::PostUndoRedo.AddRaw(this, &FModelDataEditorToolkit::ReloadSettings);

		const TSharedRef<FTabManager::FLayout> Layout = FTabManager::NewLayout("NNEModelDataEditorLayout")
			->AddArea
			(
				FTabManager::NewPrimaryArea()->SetOrientation(Orient_Vertical)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.8f)
					->AddTab(TEXT("NNEModelDataTab"), ETabState::OpenedTab)
				)
			);
		TArray<UObject*> UObjectModels;
		for (UNNEModelData* Model : Models)
		{
			UObjectModels.Add(Model);
		}
		FAssetEditorToolkit::InitAssetEditor(EToolkitMode::Standalone, {}, TEXT("NNEModelDataEditor"), Layout, true, true, UObjectModels);
	}

	void FModelDataEditorToolkit::RegisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::RegisterTabSpawners(InTabManager);

		WorkspaceMenuCategory = InTabManager->AddLocalWorkspaceMenuCategory(INVTEXT("NNE Model Data Editor"));

		InTabManager->RegisterTabSpawner("NNEModelDataTab", FOnSpawnTab::CreateRaw(this, &FModelDataEditorToolkit::SpawnTab))
			.SetDisplayName(INVTEXT("NNE Model Data"))
			.SetGroup(WorkspaceMenuCategory.ToSharedRef());
	}

	void FModelDataEditorToolkit::UnregisterTabSpawners(const TSharedRef<class FTabManager>& InTabManager)
	{
		FAssetEditorToolkit::UnregisterTabSpawners(InTabManager);
		InTabManager->UnregisterTabSpawner("NNEModelDataTab");
	}

	FText FModelDataEditorToolkit::GetToolkitName() const
	{
		check(Models.Num() > 0);
		if (Models.Num() == 1)
		{
			return GetLabelForObject(Models[0]);
		}
		return INVTEXT("Multiple NNE Models");
	}

	#define LOCTEXT_NAMESPACE "AssetEditorToolkit"
	FText FModelDataEditorToolkit::GetToolkitToolTipText() const
	{
		check(Models.Num() > 0);
		if (Models.Num() == 1)
		{
			return GetToolTipTextForObject(Models[0]);
		}
		FString ToolTipString;
		ToolTipString += LOCTEXT("ToolTipAssetLabel", "Asset").ToString();
		ToolTipString += TEXT(":\n");
		
		for (const UNNEModelData* Model : Models)
		{
			ToolTipString += TEXT("- ");
			ToolTipString += Model->GetName();
			ToolTipString += TEXT("\n");
		}
		ToolTipString.TrimEndInline();
		return FText::FromString(ToolTipString);
	}
	#undef LOCTEXT_NAMESPACE

	void FModelDataEditorToolkit::OnClose()
	{
		EmptySelectionModels->RemoveFromRoot();
		ClearSettings();
		FEditorDelegates::PostUndoRedo.Remove(PostUndoRedoHandle);
	}

	TSharedRef<SDockTab> FModelDataEditorToolkit::SpawnTab(const FSpawnTabArgs&)
	{
		TSharedPtr<SVerticalBox> ScrollContent;
		TSharedRef<SDockTab> Result = SNew(SDockTab)
			[
				SNew(SVerticalBox)
					+
					SVerticalBox::Slot()
					.AutoHeight()
					.Padding(4.0, 20.0, 4.0, 4.0)
					[
						SNew(SHorizontalBox)
							+
							SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(0, 0, 15, 0)
							[
								SNew(STextBlock).Text(FText::FromString("Runtimes"))
									.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
									.TransformPolicy(ETextTransformPolicy::ToUpper)
							]
							+
							SHorizontalBox::Slot()
							.VAlign(VAlign_Center)
							
							[
								SNew(STextBlock).Text(FText::FromString("No runtime selected. Please select at least one runtime for each model."))
									.ColorAndOpacity(FSlateColor(EStyleColor::AccentRed))
									.Visibility(this, &FModelDataEditorToolkit::GetErrorMessageVisibility)
									.Font(FAppStyle::GetFontStyle("PropertyWindow.BoldFont"))
							]
					]
					+
					SVerticalBox::Slot()
					.FillHeight(1.0)
					[
						SNew(SScrollBox)
						+
						SScrollBox::Slot()
						[
							SAssignNew(ScrollContent, SVerticalBox)
							+
							SVerticalBox::Slot()
							.AutoHeight()
							.Padding(25, 0, 0, 0)
							[
								SNew(SCheckBox)
									.IsChecked(this, &FModelDataEditorToolkit::GetCheckBoxState, AllCheckBoxIndex)
									.OnCheckStateChanged(this, &FModelDataEditorToolkit::SetCheckBoxState, AllCheckBoxIndex)
									.ToolTipText(FText::FromString(TEXT("Enables all current and future runtimes")))
									[
										SNew(STextBlock)
											.Text(FText::FromString(TEXT("All")))
											.Font(FAppStyle::GetFontStyle("PropertyWindow.NormalFont"))
											.Margin(FMargin(4, 0))

									]
							]
						]
					]
			];
		for (int RuntimeIndex = 0; RuntimeIndex < Runtimes.Num(); RuntimeIndex++)
		{
			const FRuntimeArea& Runtime = Runtimes[RuntimeIndex];
			TSharedPtr<SRuntimeExpandableArea> ExpandableArea;
			ScrollContent->AddSlot()
				.AutoHeight()
				.Padding(4.0)
				[
					SAssignNew(ExpandableArea, SRuntimeExpandableArea)
						.InitiallyCollapsed(true)
						.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header"))
						.BodyBorderImage(FAppStyle::Get().GetBrush("Brushes.Recessed"))
						.HeaderPadding(FMargin(2.f, 5.f))
						.Padding(FMargin(15.f, 1.f, 0.f, 0.f))
						.HeaderContent()
						[
							SNew(SHorizontalBox)
							+
								SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0, 0, 5, 0)
								[
									SNew(SCheckBox)
										.IsChecked(this, &FModelDataEditorToolkit::GetCheckBoxState, RuntimeIndex)
										.OnCheckStateChanged(this, &FModelDataEditorToolkit::SetCheckBoxState, RuntimeIndex)
										.ToolTipText(FText::FromString(TEXT("Enables cooking and usage of this runtime.")))
								]
								+
								SHorizontalBox::Slot()
								.AutoWidth()
								.Padding(0, 0, 10, 0)
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.Text(FText::FromString(Runtime.Name))
										.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
								]
								+
								SHorizontalBox::Slot()
								.VAlign(VAlign_Center)
								[
									SNew(STextBlock)
										.Text(this, &FModelDataEditorToolkit::GetAreaErrorMessageText, RuntimeIndex)
										.ColorAndOpacity(FSlateColor(EStyleColor::AccentRed))
										.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
								]
						]
						.BodyContent()
						[
							Runtime.DetailsView
						]
				];
			ExpandableArea->IsExpandable.BindRaw(this, &FModelDataEditorToolkit::GetAreaIsExpandable, RuntimeIndex);
		}
		return Result;
	}

	void FModelDataEditorToolkit::ReloadSettings()
	{
		using EStatus = UNNEModelData::EGetRuntimeSettingsStatus;

		ClearSettings();
		for (FRuntimeArea& Runtime : Runtimes)
		{
			TArray<UObject*> AllSettings;
			Runtime.IsVisible = true;
			for (int32 ModelIndex = 0; ModelIndex < Models.Num(); ModelIndex++)
			{
				const UNNEModelData* Model = Models[ModelIndex];
				UObject* Settings;
				EStatus Status = Model->GetRuntimeSettings(Runtime.Name, Settings);
				if (Status != EStatus::Success)
				{
					Runtime.IsVisible = false;
					switch (Status)
					{
						case EStatus::RuntimeNotFound:
						case EStatus::RuntimeUnsupported:
							break;
						case EStatus::InvalidCustomVersion:
							Runtime.ErrorMessage = INVTEXT("Error: Invalid custom version. Please try to delete and reimport the model.");
							break;
						case EStatus::NewerCustomVersion:
							Runtime.ErrorMessage = INVTEXT("Error: Asset was saved with a newer custom version than the current. Please open it with a newer engine version.");
							break;
						case EStatus::SerializationError:
							Runtime.ErrorMessage = INVTEXT("Error: Serialization failed. Please try to delete and reimport the model.");
							break;
						default:
							checkNoEntry();
							break;
					}
					break;
				}
				Settings->AddToRoot();
				AllSettings.Add(Settings);
				const FSettingsIndex SettingsIndex(Runtime.Name, ModelIndex, Settings);
				SettingsIDToIndex.Add(Settings->GetUniqueID(), SettingsIndex);
			}
			Runtime.DetailsView->SetObjects(AllSettings);
		}
	}

	void FModelDataEditorToolkit::ClearSettings()
	{
		for (const auto& Element : SettingsIDToIndex)
		{
			Element.Value.Settings->RemoveFromRoot();
		}
		SettingsIDToIndex.Reset();
	}

	void FModelDataEditorToolkit::OnRuntimePropertyChanged(const FPropertyChangedEvent& Property) const
	{
		const FString TransactionDescription = TEXT("Edit ") + Property.GetPropertyName().ToString();
		UKismetSystemLibrary::BeginTransaction(TEXT("NNE Model Data"), FText::FromString(*TransactionDescription), nullptr);
		for (int Index = 0; Index < Property.GetNumObjectsBeingEdited(); Index++)
		{
			const uint32 SettingsID = Property.GetObjectBeingEdited(Index)->GetUniqueID();
			check(SettingsIDToIndex.Contains(SettingsID));
			const FSettingsIndex& SettingsIndex = SettingsIDToIndex[SettingsID];
			UNNEModelData* Model = Models[SettingsIndex.ModelIndex];

			UKismetSystemLibrary::TransactObject(Model);
			Model->SetRuntimeSettings(SettingsIndex.RuntimeName, *SettingsIndex.Settings);
		}
		UKismetSystemLibrary::EndTransaction();
	}

	EVisibility FModelDataEditorToolkit::GetErrorMessageVisibility() const
	{
		return EmptySelectionModels->ModelIndices.IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible;
	}

	FText FModelDataEditorToolkit::GetAreaErrorMessageText(const int RuntimeIndex) const
	{
		const FRuntimeArea& Runtime = Runtimes[RuntimeIndex];
		if (!Runtime.IsVisible)
		{
			return Runtime.ErrorMessage;
		}
		return INVTEXT("");
	}

	bool FModelDataEditorToolkit::GetAreaIsExpandable(const int RuntimeIndex) const
	{
		return Runtimes[RuntimeIndex].IsVisible;
	}

	FString FModelDataEditorToolkit::ButtonIndexToRuntimeName(const int ButtonIndex) const
	{
		if (ButtonIndex == AllCheckBoxIndex)
		{
			return TEXT("All");
		}
		check(ButtonIndex >= 0);
		return Runtimes[ButtonIndex].Name;
	}

	ECheckBoxState FModelDataEditorToolkit::GetCheckBoxState(const int ButtonIndex) const
	{
		bool bAllChecked = true;
		bool bAllUnchecked = true;
		const FString RuntimeName = ButtonIndexToRuntimeName(ButtonIndex);
		for(int ModelIndex = 0; ModelIndex < Models.Num(); ModelIndex++)
		{
			const UNNEModelData* Model = Models[ModelIndex];
			const TArrayView<const FString> TargetRuntimes = Model->GetTargetRuntimes();
			const bool bIsChecked = TargetRuntimes.IsEmpty() || (TargetRuntimes.Contains(RuntimeName) && !EmptySelectionModels->ModelIndices.Contains(ModelIndex));
			bAllChecked &= bIsChecked;
			bAllUnchecked &= !bIsChecked;
		}
		const ECheckBoxState State = bAllChecked   ? ECheckBoxState::Checked :
							         bAllUnchecked ? ECheckBoxState::Unchecked :
							         /*else*/        ECheckBoxState::Undetermined;
		return State;
	}

	void FModelDataEditorToolkit::SetCheckBoxState(const ECheckBoxState, const int ButtonIndex)
	{
		// Determining the next check box state based on the current one.
		// Normaly the rule is that if the state is in Undetermined the check box gets unchecked.
		// In our case however this causes the All button to be unchecked (if it isn't allready). 
		// Since this is unexpected and not an operation that can easily be reverted we favor
		// checking the check box in this case.
		const ECheckBoxState OldState = GetCheckBoxState(ButtonIndex);
		const ECheckBoxState OldAllState = GetCheckBoxState(AllCheckBoxIndex);
		const bool bFavorSelection = ButtonIndex != AllCheckBoxIndex && OldAllState != ECheckBoxState::Unchecked;
		const bool bAddRuntime =  bFavorSelection ? OldState != ECheckBoxState::Checked : OldState == ECheckBoxState::Unchecked;

		const FString RuntimeName = ButtonIndexToRuntimeName(ButtonIndex);
		FString TransactionDescription = bAddRuntime ? TEXT("Add ") : TEXT("Remove ");
		TransactionDescription += RuntimeName;
		UKismetSystemLibrary::BeginTransaction(TEXT("NNE Model Data"), FText::FromString(*TransactionDescription), nullptr);
		for (int ModelIndex = 0; ModelIndex < Models.Num(); ModelIndex++)
		{
			UNNEModelData* Model = Models[ModelIndex];
			const TArrayView<const FString> CurrentRuntimes = Model->GetTargetRuntimes();
			const bool bIsEmptySelection = EmptySelectionModels->ModelIndices.Contains(ModelIndex);
			const bool bIsRuntimeSelected = CurrentRuntimes.IsEmpty() || CurrentRuntimes.Contains(RuntimeName);
			const bool bIsButtonChecked = bIsRuntimeSelected && !bIsEmptySelection;
			if (bAddRuntime == bIsButtonChecked)
			{
				continue;
			}
			if (bAddRuntime && bIsEmptySelection)
			{
				UKismetSystemLibrary::TransactObject(EmptySelectionModels);
				EmptySelectionModels->ModelIndices.Remove(ModelIndex);
			}
			TArray<FString> NewRuntimes;
			bool bUpdateRuntimes = bIsRuntimeSelected != bAddRuntime;
			// All check box was pressed
			if (ButtonIndex == AllCheckBoxIndex)
			{
				if (!bAddRuntime)
				{
					NewRuntimes = AllRuntimeNames;
				}
			}
			else
			{
				if (bAddRuntime)
				{
					if (!bIsEmptySelection)
					{
						NewRuntimes = CurrentRuntimes;
					}
					NewRuntimes.Add(RuntimeName);
				}
				else
				{
					if (CurrentRuntimes.IsEmpty())
					{
						NewRuntimes = UE::NNE::GetAllRuntimeNames();
						NewRuntimes.Remove(RuntimeName);
					}
					else
					{
						if (CurrentRuntimes.Num() > 1)
						{
							NewRuntimes = CurrentRuntimes;
							NewRuntimes.Remove(RuntimeName);
						}
						else
						{
							bUpdateRuntimes = false;
							UKismetSystemLibrary::TransactObject(EmptySelectionModels);
							EmptySelectionModels->ModelIndices.Add(ModelIndex);
						}
					}
				}
			}
			if (bUpdateRuntimes)
			{
				UKismetSystemLibrary::TransactObject(Model);
				Model->SetTargetRuntimes(NewRuntimes);
			}
		}
		UKismetSystemLibrary::EndTransaction();
	}
} // UE::NNEEditor::Private
