// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Toolkits/AssetEditorToolkit.h"
#include "UObject/Object.h"

#include "NNEEditorModelDataEditorToolkit.generated.h"


class UNNEModelData;

UCLASS(MinimalAPI)
class UNNEEditorEmptySelectionModels : public UObject
{
	GENERATED_BODY()
public:
	UPROPERTY()
	TArray<int> ModelIndices;
};

namespace UE::NNEEditor::Private
{
	class FModelDataEditorToolkit : public FAssetEditorToolkit
	{
	public:
		void InitEditor(const TArray<UNNEModelData*>& InModels);

		// FAssetEditorToolkit interface
		virtual void RegisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual void UnregisterTabSpawners(const TSharedRef<class FTabManager>& TabManager) override;
		virtual FName GetToolkitFName() const override { return FName("NNEModelDataEditor"); }
		virtual FText GetBaseToolkitName() const override { return INVTEXT("NNE Model Data Editor"); }
		virtual FText GetToolkitName() const override;
		virtual FText GetToolkitToolTipText() const override;
		virtual FString GetWorldCentricTabPrefix() const override { return TEXT("NNE Model Data Editor "); }
		virtual FLinearColor GetWorldCentricTabColorScale() const override { return {}; }
		virtual void OnClose() override;
		// End of FAssetEditorToolkit interface

	private:

		struct FSettingsIndex
		{
			const FString RuntimeName;
			const int32 ModelIndex;
			UObject* Settings;
		};

		struct FRuntimeArea
		{
			const FString Name;
			const TSharedRef<IDetailsView> DetailsView;
			bool IsVisible;
			FText ErrorMessage;
		};

		static constexpr int AllCheckBoxIndex = -1;

		TSharedRef<SDockTab> SpawnTab(const FSpawnTabArgs&);
		void ReloadSettings();
		void ClearSettings();
		void OnRuntimePropertyChanged(const FPropertyChangedEvent& Property) const;
		EVisibility GetErrorMessageVisibility() const;
		FText GetAreaErrorMessageText(const int RuntimeIndex) const;
		bool GetAreaIsExpandable(const int RuntimeIndex) const;
		FString ButtonIndexToRuntimeName(const int ButtonIndex) const;
		ECheckBoxState GetCheckBoxState(const int ButtonIndex) const;
		void SetCheckBoxState(const ECheckBoxState NewState, const int Index);

		TArray<UNNEModelData*> Models;
		TArray<FString> AllRuntimeNames;
		TArray<FRuntimeArea> Runtimes;
		TMap<uint32, const FSettingsIndex> SettingsIDToIndex;
		UNNEEditorEmptySelectionModels* EmptySelectionModels;
		FDelegateHandle PostUndoRedoHandle;
	};
} // UE::NNEEditor::Private
