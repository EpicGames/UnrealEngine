// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEEditorModelDataFactory.h"

#include "Editor.h"
#include "NNEEditorOnnxFileLoaderHelper.h"
#include "NNEModelData.h"
#include "Misc/Paths.h"
#include "Subsystems/ImportSubsystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(NNEEditorModelDataFactory)

UNNEModelDataFactory::UNNEModelDataFactory(const FObjectInitializer& ObjectInitializer) : UFactory(ObjectInitializer)
{
	bCreateNew = false;
	bEditorImport = true;
	SupportedClass = UNNEModelData::StaticClass();
	ImportPriority = DefaultImportPriority;
	Formats.Add("onnx;Open Neural Network Exchange Format");
}


UObject* UNNEModelDataFactory::FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled)
{
	AdditionalImportedObjects.Empty();

	FString FileExtension = FPaths::GetExtension(Filename);

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPreImport(this, InClass, InParent, InName, *FileExtension);
	
	int64 ModelFileSize = 0;
	UNNEModelData* ModelData = NewObject<UNNEModelData>(InParent, InClass, InName, Flags);
	check(ModelData)

	if (!UE::NNEEditor::Internal::OnnxFileLoaderHelper::InitUNNEModelDataFromFile(*ModelData, ModelFileSize, Filename))
	{
		GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, nullptr);
		return nullptr;
	}

	GEditor->GetEditorSubsystem<UImportSubsystem>()->BroadcastAssetPostImport(this, ModelData);

	return ModelData;
}

bool UNNEModelDataFactory::FactoryCanImport(const FString & Filename)
{
	return Filename.EndsWith(FString("onnx"));
}
