// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Factories/Factory.h"

#include "NNEEditorModelDataFactory.generated.h"

#define UE_API NNEEDITOR_API

/**
 * Class for importing and creating new UNNEModelData assets.
 *
 * The currently supported format is .onnx. The factory is invoked when a file of this format is dragged to the editor's content browser.
 */
UCLASS(MinimalAPI)
class UNNEModelDataFactory : public UFactory
{
	GENERATED_BODY()

public:
	UE_API UNNEModelDataFactory(const FObjectInitializer& ObjectInitializer);

public:
	//~ Begin UFactory Interface
	UE_API virtual UObject* FactoryCreateFile(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, const FString& Filename, const TCHAR* Parms, FFeedbackContext* Warn, bool& bOutOperationCanceled) override;
	UE_API virtual bool FactoryCanImport(const FString& Filename) override;
	//~ End UFactory Interface
};

#undef UE_API
