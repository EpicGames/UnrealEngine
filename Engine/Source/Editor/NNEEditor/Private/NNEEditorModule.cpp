// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/Paths.h"
#include "NNE.h"
#include "NNEEditorModelDataActions.h"
#include "NNEEditorOnnxModelInspector.h"

namespace UE::NNEEditor::Private::OnnxHelper
{
	void* GetSharedLibHandle(const FString& SharedLibPath)
	{
		if (!FPaths::FileExists(SharedLibPath))
		{
			UE_LOGF(LogNNE, Error, "Failed to find the NNEEditorOnnxTools shared library %ls.", *SharedLibPath);
			return nullptr;
		}

		void* SharedLibHandle = FPlatformProcess::GetDllHandle(*SharedLibPath);

		if (!SharedLibHandle)
		{
			UE_LOGF(LogNNE, Error, "Failed to load the NNEEditorOnnxTools shared library %ls.", *SharedLibPath);
		}

		return SharedLibHandle;
	}

} // namespace UE::NNEEditor::Private::OnnxHelper

class FNNEEditorModule : public IModuleInterface
{
	void* NNEEditorOnnxToolsSharedLibHandle = nullptr;

public:
	virtual void StartupModule() override
	{
#ifdef NNEEDITORONNXTOOLS_SUPPORTED
#if PLATFORM_WINDOWS && PLATFORM_CPU_ARM_FAMILY && !PLATFORM_WINDOWS_ARM64EC
		const FString ModuleDir = FPlatformProcess::GetModulesDirectory() / TEXT("arm64");
#else
		const FString ModuleDir = FPlatformProcess::GetModulesDirectory();
#endif
		const FString NNEEditorOnnxToolsSharedLibPath = FPaths::Combine(ModuleDir, TEXT(UE_STRINGIZE(NNEEDITORONNXTOOLS_SHAREDLIB_FILENAME)));
		NNEEditorOnnxToolsSharedLibHandle = UE::NNEEditor::Private::OnnxHelper::GetSharedLibHandle(NNEEditorOnnxToolsSharedLibPath);
		UE::NNEEditor::Private::OnnxModelInspectorHelper::SetupSharedLibFunctionPointer(NNEEditorOnnxToolsSharedLibHandle);
#endif // ifdef NNEEDITORONNXTOOLS_SUPPORTED

		ModelDataAssetTypeActions = MakeShared<UE::NNEEditor::Private::FModelDataAssetTypeActions>();
		FAssetToolsModule::GetModule().Get().RegisterAssetTypeActions(ModelDataAssetTypeActions.ToSharedRef());
	}

	virtual void ShutdownModule() override
	{
		if (FModuleManager::Get().IsModuleLoaded("AssetTools"))
		{
			FAssetToolsModule::GetModule().Get().UnregisterAssetTypeActions(ModelDataAssetTypeActions.ToSharedRef());
		}

		UE::NNEEditor::Private::OnnxModelInspectorHelper::ClearSharedLibFunctionPointer();

		if (NNEEditorOnnxToolsSharedLibHandle != nullptr)
		{
			FPlatformProcess::FreeDllHandle(NNEEditorOnnxToolsSharedLibHandle);
			NNEEditorOnnxToolsSharedLibHandle = nullptr;
		}
	}

private:
	TSharedPtr<UE::NNEEditor::Private::FModelDataAssetTypeActions> ModelDataAssetTypeActions;
};

IMPLEMENT_MODULE(FNNEEditorModule, NNEEditor)