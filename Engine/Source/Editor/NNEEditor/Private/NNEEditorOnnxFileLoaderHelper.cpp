// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEEditorOnnxFileLoaderHelper.h"

#include "CoreGlobals.h"
#include "HAL/FileManager.h"
#include "NNE.h"
#include "NNEEditorOnnxModelInspector.h"
#include "NNEModelData.h"
#include "Misc/FileHelper.h"
#include "Misc/Paths.h"
#include "Serialization/Archive.h"
#include "Serialization/MemoryWriter.h"

namespace UE::NNEEditor::Internal::OnnxFileLoaderHelper
{
	namespace Details
	{
		bool AppendFileToArray(const FString& Filename, TArray64<uint8>& Buffer)
		{
			FScopedLoadingState ScopedLoadingState(*Filename);

			FArchive* Reader = IFileManager::Get().CreateFileReader(*Filename, 0);
			if (!Reader)
			{
				return false;
			}

			int64 FileSize = Reader->TotalSize();
			if (FileSize > 0)
			{
				int64 BufferSizeBefore = Buffer.Num();

				Buffer.AddUninitialized(FileSize);
				Reader->Serialize(Buffer.GetData()+BufferSizeBefore, FileSize);
			}
			
			bool Success = Reader->Close();
			delete Reader;
			return Success;
		}
	}

	bool InitUNNEModelDataFromFile(UNNEModelData& ModelData, int64& ModelFileSize, const FString& Filename)
	{
		FString FileExtension = FPaths::GetExtension(Filename);
		ModelFileSize = 0;

		TArray64<uint8> OnnxData;
		if (!FFileHelper::LoadFileToArray(OnnxData, *Filename))
		{
			UE_LOGF(LogNNE, Error, "Failed to load file '%ls'", *Filename);
			return false;
		}

		if (!*FileExtension || OnnxData.IsEmpty())
		{
			return false;
		}

		TSet<FString> ExternalDataRelativeFilePaths;

		//If platform does not support parsing the onnx file, we import assuming the model does not use external data.
		if (UE::NNEEditor::Private::OnnxModelInspectorHelper::IsSharedLibFunctionPointerSetup())
		{
			if (!UE::NNEEditor::Private::OnnxModelInspectorHelper::GetExternalDataFilePaths(OnnxData, ExternalDataRelativeFilePaths))
			{
				UE_LOGF(LogNNE, Warning, "Could not parse the input model as ONNX ModelProto, model external data won't be imported if any.");
			}
		}

		ModelFileSize = OnnxData.Num();

		TMap<FString, TConstArrayView64<uint8>> AdditionalBuffers;
		TArray64<uint8> OnnxExternalDataBytesBuffer;
		TArray64<uint8> OnnxExternalDataDescriptorBuffer;

		if (!ExternalDataRelativeFilePaths.IsEmpty())
		{
			FString BasePath = FPaths::GetPath(*Filename);
			TMap<FString, int64> OnnxExternalDataDescriptor;
			int64 TotalFileSize = 0;

			for (const FString& DataRelativeFilePath : ExternalDataRelativeFilePaths)
			{
				FString DataFilePath = FPaths::Combine(BasePath, DataRelativeFilePath);
				int64 FileSize = IFileManager::Get().FileSize(*DataFilePath);
				if (FileSize < 0)
				{
					UE_LOGF(LogNNE, Error, "Failed to find file size for external data file '%ls' for Onnx Model '%ls'", *DataFilePath, *Filename);
					return false;
				}

				OnnxExternalDataDescriptor.Emplace(DataRelativeFilePath, FileSize);
				TotalFileSize += FileSize;
			}

			OnnxExternalDataBytesBuffer.Reset(TotalFileSize);
			for (const FString& DataRelativeFilePath : ExternalDataRelativeFilePaths)
			{
				FString DataFilePath = FPaths::Combine(BasePath, DataRelativeFilePath);

				if (!Details::AppendFileToArray(DataFilePath, OnnxExternalDataBytesBuffer))
				{
					UE_LOGF(LogNNE, Error, "Failed to load external data file '%ls' for Onnx Model '%ls'", *DataFilePath, *Filename);
					return false;
				}

				//Uncomment to trace loading of external onnx data from files.
				//UE_LOGF(LogNNE, Display, "Loaded external data file '%ls' while importing Onnx Model '%ls'.", *DataFilePath, *Filename);
			}
			check(OnnxExternalDataBytesBuffer.Num() == TotalFileSize);

			//Should be kept in sync with UE::NNERuntimeORT::Private::Details::OnnxExternalDataDescriptorKey & OnnxExternalDataBytesKey in NNERuntimeORT.cpp
			static FString OnnxExternalDataDescriptorKey(TEXT("OnnxExternalDataDescriptor"));
			static FString OnnxExternalDataBytesKey(TEXT("OnnxExternalDataBytes"));
			
			FMemoryWriter64 OnnxExternalDataAggregatedDescriptorWriter(OnnxExternalDataDescriptorBuffer, /*bIsPersistent = */true);

			OnnxExternalDataAggregatedDescriptorWriter << OnnxExternalDataDescriptor;

			AdditionalBuffers.Emplace(OnnxExternalDataDescriptorKey, OnnxExternalDataDescriptorBuffer);
			AdditionalBuffers.Emplace(OnnxExternalDataBytesKey, OnnxExternalDataBytesBuffer);

			ModelFileSize += OnnxExternalDataBytesBuffer.Num();
		}

		ModelData.Init(*FileExtension, OnnxData, AdditionalBuffers);

		return true;
	}

} // namespace UE::NNEEditor::Internal::OnnxFileLoaderHelper

