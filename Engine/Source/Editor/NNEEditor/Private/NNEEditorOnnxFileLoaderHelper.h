// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

class UNNEModelData;

namespace UE::NNEEditor::Internal::OnnxFileLoaderHelper
{
	bool NNEEDITOR_API InitUNNEModelDataFromFile(UNNEModelData& ModelData, int64& ModelFileSize, const FString& Filename);

} // namespace UE::NNEEditor::Internal::OnnxFileLoaderHelper
