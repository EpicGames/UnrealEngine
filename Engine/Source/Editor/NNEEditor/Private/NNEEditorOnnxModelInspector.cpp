// Copyright Epic Games, Inc. All Rights Reserved.

#include "NNEEditorOnnxModelInspector.h"

#include "NNE.h"
#include "NNEEditorOnnxTools.h"

namespace UE::NNEEditor::Private
{
namespace OnnxModelInspectorHelper
{
	class FNNEOnnxToolsWrapper
	{
		typedef NNEEditorOnnxTools_Status(*CreateExternalDataDescriptorFn)(const void* InData, const int Size, NNEEditorOnnxTools_ExternalDataDescriptor** Descriptor);
		typedef void(*ReleaseExternalDataDescriptorFn)(NNEEditorOnnxTools_ExternalDataDescriptor** Descriptor);
		typedef const char* (*GetNextExternalDataPathFn)(NNEEditorOnnxTools_ExternalDataDescriptor* Descriptor);

		CreateExternalDataDescriptorFn CreateExternalDataDescriptor{};
		ReleaseExternalDataDescriptorFn ReleaseExternalDataDescriptor{};
		GetNextExternalDataPathFn GetNextExternalDataPath{};

	public:
		static FNNEOnnxToolsWrapper& GetInstance()
		{
			static FNNEOnnxToolsWrapper Instance;
			return Instance;
		}

		void SetupSharedLibFunctionPointer(void* SharedLibHandle)
		{
			if (SharedLibHandle)
			{
				CreateExternalDataDescriptor = reinterpret_cast<CreateExternalDataDescriptorFn>(FPlatformProcess::GetDllExport(SharedLibHandle, TEXT("NNEEditorOnnxTools_CreateExternalDataDescriptor")));
				ReleaseExternalDataDescriptor = reinterpret_cast<ReleaseExternalDataDescriptorFn>(FPlatformProcess::GetDllExport(SharedLibHandle, TEXT("NNEEditorOnnxTools_ReleaseExternalDataDescriptor")));
				GetNextExternalDataPath = reinterpret_cast<GetNextExternalDataPathFn>(FPlatformProcess::GetDllExport(SharedLibHandle, TEXT("NNEEditorOnnxTools_GetNextExternalDataPath")));
			}
			if (CreateExternalDataDescriptor == nullptr ||
				ReleaseExternalDataDescriptor == nullptr ||
				GetNextExternalDataPath == nullptr)
			{
				UE_LOGF(LogNNE, Warning, "Could not find required function pointers in NNEEditorOnnxTools shared library.");
			}
		}

		void ClearSharedLibFunctionPointer()
		{
			CreateExternalDataDescriptor = nullptr;
			ReleaseExternalDataDescriptor = nullptr;
			GetNextExternalDataPath = nullptr;
		}

		bool IsSharedLibFunctionPointerSetup()
		{
			return (CreateExternalDataDescriptor != nullptr &&
					ReleaseExternalDataDescriptor != nullptr &&
					GetNextExternalDataPath != nullptr);
		}

		bool GetExternalDataFilePaths(TConstArrayView<uint8> ONNXData, TSet<FString>& ExternalDataFilePaths)
		{
			ExternalDataFilePaths.Reset();

			if (!IsSharedLibFunctionPointerSetup())
				return false;

			NNEEditorOnnxTools_ExternalDataDescriptor* ExternalDataDescriptor = nullptr;
			const NNEEditorOnnxTools_Status Result = CreateExternalDataDescriptor(ONNXData.GetData(), ONNXData.Num(), &ExternalDataDescriptor);

			if (Result == NNEEditorOnnxTools_Status::Ok)
			{
				const char* ExternalPathFromLib = GetNextExternalDataPath(ExternalDataDescriptor);
				while (ExternalPathFromLib != nullptr)
				{
					const FString ExternalPath = StringCast<TCHAR>(ExternalPathFromLib).Get();
					ExternalDataFilePaths.Emplace(ExternalPath);
					ExternalPathFromLib = GetNextExternalDataPath(ExternalDataDescriptor);
				}

				ReleaseExternalDataDescriptor(&ExternalDataDescriptor);
			}
			else
			{
				return false;
			}

			return true;

		}
	};

	void SetupSharedLibFunctionPointer(void* SharedLibHandle)
	{
		FNNEOnnxToolsWrapper::GetInstance().SetupSharedLibFunctionPointer(SharedLibHandle);
	}

	void ClearSharedLibFunctionPointer()
	{
		FNNEOnnxToolsWrapper::GetInstance().ClearSharedLibFunctionPointer();
	}

	bool IsSharedLibFunctionPointerSetup()
	{
		return FNNEOnnxToolsWrapper::GetInstance().IsSharedLibFunctionPointerSetup();
	}

	bool GetExternalDataFilePaths(TConstArrayView<uint8> ONNXData, TSet<FString>& ExternalDataFilePaths)
	{
		return FNNEOnnxToolsWrapper::GetInstance().GetExternalDataFilePaths(ONNXData, ExternalDataFilePaths);
	}

} // namespace OnnxModelInspectorHelper

} // namespace UE::NNEEditor::Private
