// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"

namespace UE::NNEEditor::Private::OnnxModelInspectorHelper
{
	void SetupSharedLibFunctionPointer(void* SharedLibHandle);
	void ClearSharedLibFunctionPointer();
	bool IsSharedLibFunctionPointerSetup();
	bool GetExternalDataFilePaths(TConstArrayView<uint8> ONNXData, TSet<FString>& ExternalDataFilePaths);

} // namespace UE::NNEEditor::Private::OnnxModelInspectorHelper
