// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if defined(NNEEDITORONNXTOOLS_EXPORT)
    #if defined(_MSC_VER)
        //  Microsoft 
        #define NNEEDITORONNXTOOLS_API __declspec(dllexport)
    #elif defined(__GNUC__)
        //  GCC
        #define NNEEDITORONNXTOOLS_API __attribute__((visibility("default")))
    #else
	    #define NNEEDITORONNXTOOLS_API unsupported_platform
    #endif
#else
    #define NNEEDITORONNXTOOLS_API
#endif

#ifndef UE_NNEEDITORONNXTOOLS
#include <cstdint>
typedef uint8_t uint8;
#else
#include "CoreMinimal.h"
#endif

enum class NNEEditorOnnxTools_Status : uint8
{
	Ok = 0,
	Fail_CannotParseAsModelProto
};

struct NNEEditorOnnxTools_ExternalDataDescriptor;

extern "C"
{

NNEEDITORONNXTOOLS_API NNEEditorOnnxTools_Status NNEEditorOnnxTools_CreateExternalDataDescriptor(const void* InData, const int Size, NNEEditorOnnxTools_ExternalDataDescriptor** Descriptor);

NNEEDITORONNXTOOLS_API void NNEEditorOnnxTools_ReleaseExternalDataDescriptor(NNEEditorOnnxTools_ExternalDataDescriptor** Descriptor);

NNEEDITORONNXTOOLS_API const char* NNEEditorOnnxTools_GetNextExternalDataPath(NNEEditorOnnxTools_ExternalDataDescriptor* Descriptor);

}